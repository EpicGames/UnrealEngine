[InputPlatformSettings_Windows InputPlatformSettings]
+HardwareDevices=(InputClassName="FWinDualShock",HardwareDeviceIdentifier="DualSense",PrimaryDeviceType=Gamepad,SupportedFeaturesMask=1996)
+HardwareDevices=(InputClassName="FWinDualShock",HardwareDeviceIdentifier="DualShock4",PrimaryDeviceType=Gamepad,SupportedFeaturesMask=332)
