// Copyright Epic Games, Inc. All Rights Reserved.

#include "WinDualShock.h"
#include "Containers/CircularQueue.h"
#include "Engine/Engine.h"
#include "Misc/CoreDelegates.h"
#include "Misc/MessageDialog.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"
#include "GenericPlatform/IInputInterface.h"
#include "IInputDeviceModule.h"
#include "IInputDevice.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/ConfigCacheIni.h"
#include "AudioDevice.h"

#if DUALSHOCK4_SUPPORT
#if PLATFORM_WINDOWS
#include "Windows/AllowWindowsPlatformTypes.h"
#include "Windows/AllowWindowsPlatformAtomics.h"

THIRD_PARTY_INCLUDES_START
#include <Audioclient.h>					// WASAPI api
#include <winreg.h>
#include <xaudio2.h>
THIRD_PARTY_INCLUDES_END
#endif // PLATFORM_WINDOWS

#include <pad.h>
#include <pad_audio.h>

#if PLATFORM_WINDOWS
#include "Windows/HideWindowsPlatformAtomics.h"
#include "Windows/HideWindowsPlatformTypes.h"
#endif

#include LIBSCEPAD_PLATFORM_INCLUDE
#include "Internationalization/Regex.h"
#endif // #if DUALSHOCK4_SUPPORT

DEFINE_LOG_CATEGORY_STATIC(LogWinDualShock, Log, All);

#if DUALSHOCK4_SUPPORT
#include "WinDualShockExternalEndpoints.h"
#include "WinDualShockControllers.h"

static FName InputClassName = FName("FWinDualShock");

class FWinDualShock : public IInputDevice
{
public:
	FWinDualShock(const TSharedRef< FGenericApplicationMessageHandler >& InMessageHandler)
		: MessageHandler(InMessageHandler)
	{
		// Configure touch and mouse events
		bool bDSTouchEvents = false;
		bool bDSTouchAxisButtons = false;
		bool bDSMouseEvents = false;
		bool bDSMotionEvents = false;

		if (GConfig)
		{
			// Configure SonyControllers to emit touch events from the touchpad if the application wants them.
			GConfig->GetBool(TEXT("SonyController"), TEXT("bDSTouchEvents"), bDSTouchEvents, GEngineIni);

			// Configure SonyControllers to emit axis events from the touchpad if the application wants them.
			GConfig->GetBool(TEXT("SonyController"), TEXT("bDSTouchAxisButtons"), bDSTouchAxisButtons, GEngineIni);

			// Configure SonyControllers to emit mouse events from the touchpad if the application wants them
			GConfig->GetBool(TEXT("SonyController"), TEXT("bDSMouseEvents"), bDSMouseEvents, GEngineIni);

			// Configure SonyControllers to emit motion events if the application wants them
			GConfig->GetBool(TEXT("SonyController"), TEXT("bDSMotionEvents"), bDSMotionEvents, GEngineIni);
		}

		// On Windows, we need to compensate the padspeaker and haptics volume
		// by the platform headroom attenuation

		FAudioThread::RunCommandOnAudioThread([this]()
		{
			float DSPadSpeakerGain = 1.f;
			float DSHeadphonesGain = 1.f;
			float DSMicrophoneGain = 1.f;
			float DSOutputGain = 1.f;

			// Configure SonyControllers gain (between 0.0-1.0, default 1.0)
			GConfig->GetFloat(TEXT("SonyController"), TEXT("DSPadSpeakerGain"), DSPadSpeakerGain, GEngineIni);
			GConfig->GetFloat(TEXT("SonyController"), TEXT("DSHeadphonesGain"), DSHeadphonesGain, GEngineIni);
			GConfig->GetFloat(TEXT("SonyController"), TEXT("DSMicrophoneGain"), DSMicrophoneGain, GEngineIni);

			if (GEngine)
			{
				if (auto MainAudioDevice = GEngine->GetMainAudioDevice())
				{
					DSOutputGain = 1.0f / FMath::Clamp(MainAudioDevice->GetPlatformAudioHeadroom(), TNumericLimits<float>::Min(), 1.f);
				}
			}
			Controllers.SetAudioGain(DSPadSpeakerGain, DSHeadphonesGain, DSMicrophoneGain, DSOutputGain);
		});

		Controllers.SetEmitTouchEvents(bDSTouchEvents);
		Controllers.SetEmitTouchAxisEvents(bDSTouchAxisButtons);
		Controllers.SetEmitMouseEvents(bDSMouseEvents);
		Controllers.SetEmitMotionEvents(bDSMotionEvents);

		for (int32 UserIndex = 0; UserIndex < SCE_USER_SERVICE_MAX_LOGIN_USERS; UserIndex++)
		{
			FWinDualShockControllers::FControllerState& State = Controllers.GetControllerState(UserIndex);
			if (State.RefCount == 0)
			{
				Controllers.ConnectStateToUser(SCE_USER_SERVICE_STATIC_USER_ID_1 + UserIndex, UserIndex);
			}
		}
	}

	virtual ~FWinDualShock() override
	{
	}

	virtual EInputDeviceThreadAffinity GetThreadAffinity() const override { return EInputDeviceThreadAffinity::InputThreadSafe; }

	virtual void Tick(float DeltaTime) override
	{
	}

	virtual bool IsGamepadAttached() const override
	{
		return Controllers.IsGamepadAttached();
	}

	virtual void SendControllerEvents() override
	{
		for (int32 UserIndex = 0; UserIndex < SCE_USER_SERVICE_MAX_LOGIN_USERS; UserIndex++)
		{
			// On Windows all controllers exist and can be opened, but until their state changes to connected
			// (which is updated asynchronously on an internal thread in libScePad) we don't know the controller type.
			// That's why checking querying the type in ConnectStateToUser is not enough and we need this lazy update.
			if (Controllers.GetControllerTypeIdentifier(UserIndex) == ESonyControllerType::None)
			{
				Controllers.RefreshControllerType(UserIndex);
			}

			// Don't send events until we have the controller type as we need to set the device scope up.
			if (Controllers.GetControllerTypeIdentifier(UserIndex) != ESonyControllerType::None)
			{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
			FInputDeviceScope InputScope(this, InputClassName, UserIndex, Controllers.GetControllerTypeIdentifierName(UserIndex));
PRAGMA_ENABLE_DEPRECATION_WARNINGS
				Controllers.SendControllerEvents(UserIndex, MessageHandler);
			}
		}
		UpdateAudioDevices();
	}

	void SetChannelValue(int32 ControllerId, FForceFeedbackChannelType ChannelType, float Value)
	{
		Controllers.SetForceFeedbackChannelValue(ControllerId, ChannelType, Value);
	}

	void SetChannelValues(int32 ControllerId, const FForceFeedbackValues& Values)
	{
		Controllers.SetForceFeedbackChannelValues(ControllerId, Values);
	}

	virtual void SetMessageHandler(const TSharedRef< FGenericApplicationMessageHandler >& InMessageHandler) override
	{
		MessageHandler = InMessageHandler;
}

	virtual bool Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override
	{
		return false;
	}

	void SetDeviceProperty(int32 ControllerId, const FInputDeviceProperty* Property) override
	{
		Controllers.SetDeviceProperty(ControllerId, Property);
	}

	void AddAudioDevice(const FDeviceKey& DeviceKey, TSharedRef<IWinDualShockAudioDevice> Device)
	{
		if (!DeviceMap.Contains(DeviceKey))
		{
			DeviceMap.Add(DeviceKey, Device);
		}
	}

	template<typename LAMBDA>
	TSharedRef<IWinDualShockAudioDevice> GetAudioDevice(const FDeviceKey& DeviceKey, LAMBDA&& CreateDevice)
	{
		if (!DeviceMap.Contains(DeviceKey))
		{
			DeviceMap.Add(DeviceKey, CreateDevice());
		}
		return DeviceMap[DeviceKey];
	}

	float GetOutputGain()
	{
		return Controllers.GetOutputGain();
	}

	FString GetContainerRegistryPath(int32 UserIndex)
	{
		return Controllers.GetContainerRegistryPath(UserIndex);
	}

private:
	void UpdateAudioDevices()
	{
		TArray<FDeviceKey> DevicesToDelete;
		for (auto& Device : DeviceMap)
		{
			if (Device.Value->GetEndpointCount() == 0)
			{
				DevicesToDelete.Add(Device.Key);
				continue;
			}
			if (Controllers.GetSupportsAudio(Device.Key.UserIndex))
			{
				Device.Value->SetupAudio(Controllers.GetControllerTypeIdentifier(Device.Key.UserIndex));
			}
			else
			{
				Device.Value->TearDownAudio();
			}
		}
		for (FDeviceKey& Key : DevicesToDelete)
		{
			DeviceMap.Remove(Key);
		}
	}

	// handler to send all messages to
	TSharedRef<FGenericApplicationMessageHandler> MessageHandler;

	// the object that encapsulates all the controller logic
	FWinDualShockControllers Controllers;

	// active audio devices
	TMap<FDeviceKey, TSharedRef<IWinDualShockAudioDevice>> DeviceMap;
};

// IWinDualShockAudioDevice implementation
class FWinDualShockAudioDeviceImpl : public IWinDualShockAudioDevice
{
public:
	FWinDualShockAudioDeviceImpl(FWinDualShock* InInputDevice, const FDeviceKey& InDeviceKey) 
		: IWinDualShockAudioDevice(InDeviceKey), InputDevice(InInputDevice)
	{
	}

	virtual ~FWinDualShockAudioDeviceImpl()
	{
		TearDownAudioInternal();
	}

	void SetInputDevice(FWinDualShock* InInputDevice)
	{
		InputDevice = InInputDevice;
	}

	virtual void PushAudio(EWinDualShockPortType PortType, const TArrayView<const float>& InAudio, const int32& NumChannels)
	{
		if (InAudio.Num() == 0 || !bAudioIsSetup.load() || !InputDevice)
			return;

		float OutputGain = InputDevice->GetOutputGain();
		if (PortType == EWinDualShockPortType::PadSpeakers)
		{
			if (!AudioDevice.IsStreamStarted() && AudioPadSpeakerBuffer->Count() >= EWinDualShockDefaults::NumFrames)
			{
				if (AudioDevice.NumOutputChannels == 2)
					SubmitBuffer<2>();
				else
					SubmitBuffer<4>();
				AudioDevice.StartStream();
			}

			check(NumChannels == 1 || NumChannels == 2);
			check(AudioPadSpeakerBuffer);

			if (NumChannels == 1)
			{
				// convert to stereo
				const float* s = InAudio.GetData();

				for (int32 i = 0; i < InAudio.Num(); i++)
				{
					float left = *s;
					float right = *s++;
					AudioPadSpeakerBuffer->Enqueue(left * OutputGain);
					AudioPadSpeakerBuffer->Enqueue(right * OutputGain);
				}
			}
			else
			{

				const float* s = InAudio.GetData();
				for (int32 i = 0; i < (InAudio.Num() / 2); i++)
				{
					float left = *s++;
					float right = *s++;
					AudioPadSpeakerBuffer->Enqueue(left * OutputGain);
					AudioPadSpeakerBuffer->Enqueue(right * OutputGain);
				}
			}
		}
		else if (PortType == EWinDualShockPortType::Vibration)
		{
			if (!AudioDevice.IsStreamStarted() && AudioVibrationBuffer->Count() >= EWinDualShockDefaults::NumFrames)
			{
				if (AudioDevice.NumOutputChannels == 2)
					SubmitBuffer<2>();
				else
					SubmitBuffer<4>();
				AudioDevice.StartStream();
			}

			check(NumChannels == 2);
			check(AudioVibrationBuffer);

			const float* s = InAudio.GetData();
			for (int32 i = 0; i < (InAudio.Num() / 2); i++)
			{
				float left = *s++;
				float right = *s++;

				AudioVibrationBuffer->Enqueue(left * OutputGain);
				AudioVibrationBuffer->Enqueue(right * OutputGain);
			}
		}
	}

	template<int channels>
	void SubmitBuffer()
	{
		float value;
		float* pOutput = AudioDevice.OutputBuffer.GetData();

		for (int32 frame = 0; frame < EWinDualShockDefaults::NumFrames; frame++)
		{
			value = 0;
			AudioPadSpeakerBuffer->Dequeue(value);
			*pOutput++ = value;

			value = 0;
			AudioPadSpeakerBuffer->Dequeue(value);
			*pOutput++ = value;

			if (channels == 4)
			{
				value = 0;
				AudioVibrationBuffer->Dequeue(value);
				*pOutput++ = value;

				value = 0;
				AudioVibrationBuffer->Dequeue(value);
				*pOutput++ = value;
			}
		}
		AudioDevice.SubmitBuffer(CallbackInfo);
	}

	virtual void SetupAudio(ESonyControllerType ControllerType)
	{
		if (bAudioIsSetup.load() || !InputDevice)
		{
			return;
		}

		FString RegPath = InputDevice->GetContainerRegistryPath(DeviceKey.UserIndex);
		if (RegPath.IsEmpty())
		{
			static bool bWarningShown = false;
			if (!bWarningShown)
			{
				UE_LOGF(LogWinDualShock, Warning, "This version of libscepad doesn't provide a registry container query API, no audio will be setup.");
				bWarningShown = true;
			}
			return;
		}

		HKEY hRegKey;
		if (RegOpenKeyEx(HKEY_LOCAL_MACHINE, FTCHARToWChar(*RegPath).Get(), 0, KEY_QUERY_VALUE, &hRegKey) != ERROR_SUCCESS)
		{
			UE_LOGF(LogWinDualShock, Warning, "Could not query registry.");
			return;
		}

		FString OutputDeviceId, InputDeviceId;

		int32 ValueIndex = 0;

		do
		{
			DWORD ValueNameLength = 2048;
			WCHAR ValueName[2048];

			auto res = RegEnumValue(hRegKey, ValueIndex++, ValueName, &ValueNameLength, NULL, NULL, NULL, NULL);

			if (res != ERROR_SUCCESS)
			{
				break;
			}

			FString Key = WCHAR_TO_TCHAR(ValueName);
			FString Prefix = TEXT("SWD\\MMDEVAPI\\");

			if (Key.StartsWith(TEXT("SWD\\MMDEVAPI\\{0.0.0.00000000}")))
			{
				OutputDeviceId = Key.RightChop(Prefix.Len());
			}
			else if (Key.StartsWith(TEXT("SWD\\MMDEVAPI\\{0.0.1.00000000}")))
			{
				InputDeviceId = Key.RightChop(Prefix.Len());
			}

			if (!OutputDeviceId.IsEmpty() && !InputDeviceId.IsEmpty())
				break;

		} while (ValueIndex < 256); // Absurdly high number of values gives us an out

		RegCloseKey(hRegKey);

		if (OutputDeviceId.IsEmpty() || InputDeviceId.IsEmpty())
		{
			UE_LOGF(LogWinDualShock, Warning, "Couldn't locate device id for user=%d", DeviceKey.UserIndex);
		}

		if (!AudioDevice.Initialize(OutputDeviceId, ControllerType))
		{
			UE_LOGF(LogWinDualShock, Warning, "Could not initialize audio for user=%d", DeviceKey.UserIndex);
			return;
		}

		uint32 NumOutputChannels = AudioDevice.NumOutputChannels;
		CallbackInfo.Context = this;
		AudioPadSpeakerBuffer = new TCircularQueue<float>(EWinDualShockDefaults::NumFrames * NumOutputChannels * EWinDualShockDefaults::QueueDepth + 1);
		AudioVibrationBuffer = new TCircularQueue<float>(EWinDualShockDefaults::NumFrames * NumOutputChannels * EWinDualShockDefaults::QueueDepth + 1);
		bAudioIsSetup.store(true, std::memory_order_release);
	}

	virtual void TearDownAudio()
	{
		TearDownAudioInternal();
	}

protected:
	struct FCallbackInfo
	{
		FWinDualShockAudioDeviceImpl* Context;
	};

	class FXAudio2VoiceCallback final : public IXAudio2VoiceCallback
	{
	public:
		FXAudio2VoiceCallback() {}
		~FXAudio2VoiceCallback() {}

	private:
		void STDCALL OnVoiceProcessingPassStart(UINT32 BytesRequired) {}
		void STDCALL OnVoiceProcessingPassEnd() {}
		void STDCALL OnStreamEnd() {}
		void STDCALL OnBufferStart(void* BufferContext) {}
		void STDCALL OnLoopEnd(void* BufferContext) {}
		void STDCALL OnVoiceError(void* BufferContext, HRESULT Error) {}

		void STDCALL OnBufferEnd(void* BufferContext)
		{
			FCallbackInfo* ud = (FCallbackInfo*)BufferContext;
			FWinDualShockAudioDeviceImpl* us = ud->Context;

			if (us->AudioDevice.NumOutputChannels == 2)
				us->SubmitBuffer<2>();
			else
				us->SubmitBuffer<4>();
		}
	};

	struct FXAudio2Device
	{
	public:
		bool Initialize(const FString& OutputDeviceId, ESonyControllerType ControllerType)
		{
			HRESULT hr;
			hr = XAudio2Create(&Context, 0, XAUDIO2_DEFAULT_PROCESSOR);
			if (FAILED(hr))
			{
				if (hr != LastInitFailureHr)
				{
					UE_LOGF(LogWinDualShock, Warning, "Could not create XAudio2 instance hr:%x", hr);
					LastInitFailureHr = hr;
				}
				Shutdown();
				return false;
			}

			NumOutputChannels = ControllerType == ESonyControllerType::DualSense ? 4 : 2;
			hr = Context->CreateMasteringVoice(
				&MasterVoice,
				NumOutputChannels,
				EWinDualShockDefaults::SampleRate,
				0,
				TCHAR_TO_WCHAR(*OutputDeviceId),
				nullptr,
				AudioCategory_GameEffects
			);
			if (FAILED(hr))
			{
				if (hr != LastInitFailureHr)
				{
					UE_LOGF(LogWinDualShock, Warning, "Could not create XAudio2 mastering voice hr:%x", hr);
					LastInitFailureHr = hr;
				}
				Shutdown();
				return false;
			}

			hr = Context->StartEngine();
			if (FAILED(hr))
			{
				if (hr != LastInitFailureHr)
				{
					UE_LOGF(LogWinDualShock, Warning, "Could not start XAudio2 engine hr:%x", hr);
					LastInitFailureHr = hr;
				}
				Shutdown();
				return false;
			}

			WAVEFORMATEX Format = { 0 };
			Format.nChannels = NumOutputChannels;
			Format.nSamplesPerSec = EWinDualShockDefaults::SampleRate;
			Format.wFormatTag = WAVE_FORMAT_IEEE_FLOAT;
			Format.nAvgBytesPerSec = Format.nSamplesPerSec * sizeof(float) * Format.nChannels;
			Format.nBlockAlign = sizeof(float) * Format.nChannels;
			Format.wBitsPerSample = sizeof(float) * 8;
			hr = Context->CreateSourceVoice(
				&SourceVoice,
				&Format,
				XAUDIO2_VOICE_NOPITCH,
				XAUDIO2_DEFAULT_FREQ_RATIO,
				&OutputVoiceCallback);
			if (FAILED(hr))
			{
				if (hr != LastInitFailureHr)
				{
					UE_LOGF(LogWinDualShock, Warning, "Could not create XAudio2 source voice hr:%x", hr);
					LastInitFailureHr = hr;
				}
				Shutdown();
				return false;
			}

			OutputBuffer.SetNumZeroed(EWinDualShockDefaults::NumFrames * NumOutputChannels);

			return true;
		}

		void Shutdown()
		{
			if (Context)
			{
				Context->StopEngine();
			}
			if (SourceVoice)
			{
				SourceVoice->DestroyVoice();
				SourceVoice = nullptr;
			}
			if (MasterVoice)
			{
				MasterVoice->DestroyVoice();
				MasterVoice = nullptr;
			}
			SAFE_RELEASE(Context);

			if (XAudio2Dll)
			{
				if (!FreeLibrary(XAudio2Dll))
				{
					UE_LOGF(LogAudioMixer, Warning, "FXAudio2Device failed to free XAudio2 Dll");
				}

				XAudio2Dll = nullptr;
			}

			bStreamStarted = false;
		}

		void SubmitBuffer(FCallbackInfo& InCallbackInfo) const
		{
			XAUDIO2_BUFFER XAudio2Buffer = { 0 };
			XAudio2Buffer.AudioBytes = EWinDualShockDefaults::NumFrames * NumOutputChannels * sizeof(float);
			XAudio2Buffer.pAudioData = (const BYTE*)OutputBuffer.GetData();
			XAudio2Buffer.pContext = &InCallbackInfo;

			// Submit buffer to the output streaming voice
			HRESULT hr;
			hr = SourceVoice->SubmitSourceBuffer(&XAudio2Buffer);
			if (FAILED(hr))
			{
				UE_LOGF(LogAudioMixer, Warning, "SubmitSourceBuffer failed %x", hr);
			}
		}

		void StartStream()
		{
			if (!bStreamStarted)
			{
				bStreamStarted = true;
				SourceVoice->Start();
			}
		}

		bool IsStreamStarted() const
		{
			return bStreamStarted;
		}

		bool IsInitialized() const
		{
			return Context != nullptr;
		}

		TArray<float>			OutputBuffer;
		uint32					NumOutputChannels;

	private:
		HMODULE					XAudio2Dll = nullptr;
		IXAudio2*				Context = nullptr;
		IXAudio2MasteringVoice*	MasterVoice = nullptr;
		IXAudio2SourceVoice*	SourceVoice = nullptr;
		bool					bStreamStarted = false;
		FXAudio2VoiceCallback	OutputVoiceCallback;

		// Last HRESULT logged from a failed Initialize() attempt. Used to dedupe warnings
		HRESULT					LastInitFailureHr = S_OK;
	};

	std::atomic_bool		bAudioIsSetup{ false };
	FXAudio2Device			AudioDevice;
	FCallbackInfo			CallbackInfo;
	TCircularQueue<float>*	AudioPadSpeakerBuffer;
	TCircularQueue<float>*	AudioVibrationBuffer;
	FWinDualShock*			InputDevice = nullptr;

private:

	void TearDownAudioInternal()
	{
		if (bAudioIsSetup.load())
		{
			bAudioIsSetup.store(false, std::memory_order_release);
			if (AudioDevice.IsInitialized())
			{
				AudioDevice.Shutdown();
			}

			if (AudioPadSpeakerBuffer)
			{
				delete AudioPadSpeakerBuffer;
				AudioPadSpeakerBuffer = NULL;
			}

			if (AudioVibrationBuffer)
			{
				delete AudioVibrationBuffer;
				AudioVibrationBuffer = NULL;
			}
		}
	}
};

template<EWinDualShockPortType PortType>
class FExternalDualShockEndpointFactory : public IAudioEndpointFactory
{
public:
	FExternalDualShockEndpointFactory(TMap<FDeviceKey, TSharedRef<FWinDualShockAudioDeviceImpl>>& InEarlyDeviceMap)
		: EarlyDeviceMap(InEarlyDeviceMap)
	{
	}

	void Register()
	{
		IAudioEndpointFactory::RegisterEndpointType(this);
		bIsImplemented = true;
	}

	~FExternalDualShockEndpointFactory()
	{
		IAudioEndpointFactory::UnregisterEndpointType(this);
	}

	virtual FName GetEndpointTypeName() override
	{
		if (PortType == EWinDualShockPortType::Vibration)
		{
			static FName VibrationPortName = TEXT("Vibration Output");
			return VibrationPortName;
		}
		else if (PortType == EWinDualShockPortType::PadSpeakers)
		{
			static FName PadSpeakersPortName = TEXT("Pad Speaker Output");
			return PadSpeakersPortName;
		}
		else
		{
			checkNoEntry();
			return FName();
		}
	}

	virtual TUniquePtr<IAudioEndpoint> CreateNewEndpointInstance(const FAudioPluginInitializationParams& InitInfo, const IAudioEndpointSettingsProxy& InitialSettings) override
	{
		const FDualShockExternalEndpointSettings& Settings = DowncastSoundfieldRef<const FDualShockExternalEndpointSettings>(InitialSettings);
		FDeviceKey DeviceKey = { InitInfo.AudioDevicePtr->DeviceID, Settings.ControllerIndex };
		auto AudioDeviceCreateFn = [this, DeviceKey]()
		{
			return MakeShareable(new FWinDualShockAudioDeviceImpl(InputDevice, DeviceKey));
		};
		TSharedRef<IWinDualShockAudioDevice> Device = 
			InputDevice ? InputDevice->GetAudioDevice(DeviceKey, AudioDeviceCreateFn) : GetEarlyAudioDevice(DeviceKey, AudioDeviceCreateFn);
		FExternalWinDualShockEndpoint<PortType>* Endpoint = new FExternalWinDualShockEndpoint<PortType>(Device);
		if (!Endpoint->IsEndpointAllowed())
		{
			UE_LOGF(LogWinDualShock, Warning, "Only one instance of %ls endpoint submit is allowed.  Additional instances ignored.", *GetEndpointTypeName().ToString());
		}
		return TUniquePtr<IAudioEndpoint>(Endpoint);
	}

	virtual UClass* GetCustomSettingsClass() const override
	{
		return UDualShockExternalEndpointSettings::StaticClass();
	}


	virtual const UAudioEndpointSettingsBase* GetDefaultSettings() const override
	{
		return GetDefault<UDualShockExternalEndpointSettings>();
	}

	void SetInputDevice(FWinDualShock* InInputDevice)
	{
		InputDevice = InInputDevice;
	}

private:
	template<typename LAMBDA>
	TSharedRef<IWinDualShockAudioDevice> GetEarlyAudioDevice(const FDeviceKey& DeviceKey, LAMBDA&& CreateDevice)
	{
		if (!EarlyDeviceMap.Contains(DeviceKey))
		{
			EarlyDeviceMap.Add(DeviceKey, CreateDevice());
		}
		return EarlyDeviceMap[DeviceKey];
	}

	FWinDualShock* InputDevice = nullptr;
	TMap<FDeviceKey, TSharedRef<FWinDualShockAudioDeviceImpl>>& EarlyDeviceMap;
};

#endif

class FWinDualShockPlugin : public IInputDeviceModule
{
#if DUALSHOCK4_SUPPORT
	// audio devices allocated early before FWinDualShock created
	TMap<FDeviceKey, TSharedRef<FWinDualShockAudioDeviceImpl>>				EarlyDeviceMap;

	// exported endpoints
	FExternalDualShockEndpointFactory<EWinDualShockPortType::PadSpeakers>	PadSpeakerEndpoint;
	FExternalDualShockEndpointFactory<EWinDualShockPortType::Vibration>		VibrationEndpoint;

public:
	FWinDualShockPlugin()
		: PadSpeakerEndpoint(EarlyDeviceMap), VibrationEndpoint(EarlyDeviceMap)
	{
	}

	virtual ~FWinDualShockPlugin() override
	{
	}

	virtual void StartupModule() override
	{
		IInputDeviceModule::StartupModule();

		// We only need to register the audio submix endpoints if this input API is preferred.
		// Otherwise, we may be creating endpoints which are not useable. 
		if (FGenericPlatformMisc::IsPreferredInputDevice(GetPreferredDeviceAPIString()))
		{
			PadSpeakerEndpoint.Register();
			VibrationEndpoint.Register();	
		}
	}
#endif

	virtual TSharedPtr< class IInputDevice > CreateInputDevice(const TSharedRef< FGenericApplicationMessageHandler >& InMessageHandler) override
	{
#if DUALSHOCK4_SUPPORT
		FWinDualShock* InputDevice = new FWinDualShock(InMessageHandler);
		PadSpeakerEndpoint.SetInputDevice(InputDevice);
		VibrationEndpoint.SetInputDevice(InputDevice);
		// transfer ownership of any audio devices created before FWinDualShock
		for (auto& Device : EarlyDeviceMap)
		{
			Device.Value->SetInputDevice(InputDevice);
			InputDevice->AddAudioDevice(Device.Key, Device.Value);
		}
		EarlyDeviceMap.Reset();
		return TSharedPtr< class IInputDevice >(InputDevice);
#else
		return TSharedPtr< class IInputDevice >(nullptr);
#endif
	}

	virtual const TCHAR* GetPreferredDeviceAPIString() const override
	{
		static const TCHAR* APIName = TEXT("WinDualShock"); 
		return APIName;
	}
};

IMPLEMENT_MODULE(FWinDualShockPlugin, WinDualShock)
