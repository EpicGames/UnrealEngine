// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

enum class ESonyControllerType : unsigned int;

namespace Audio
{
	/**
	 * Typed identifier for Audio Device Id
	 */
	using FDeviceId = uint32;
}

/**
 * Supported endpoint types
 */
enum EWinDualShockPortType
{
	PadSpeakers,
	Vibration,	// ignored on DualShock controllers
	WinDualShockPortType_Count
};

/**
 * Unique DeviceID,UserIndex pair
 */
struct FDeviceKey
{
	Audio::FDeviceId DeviceID;
	int32 UserIndex;

	inline bool operator ==(const FDeviceKey& Other) const
	{
		return DeviceID == Other.DeviceID && UserIndex == Other.UserIndex;
	}
};

static inline uint32 GetTypeHash(const FDeviceKey& InKey)
{
	return HashCombine(InKey.DeviceID, InKey.UserIndex);
}

/**
 * Interface to audio device in a DualShock or DualSense controller.
 * One of these created for each active FAudioDevice::DeviceID since
 * endpoints will be created for each of those devices that will ultimately
 * be routed here.
 */
class IWinDualShockAudioDevice
{
public:
	IWinDualShockAudioDevice(const FDeviceKey& InDeviceKey)
		: DeviceKey(InDeviceKey)
	{
	}

	/**
	 * Initialize an audio device for the controller type specified
	 */
	virtual void	SetupAudio(ESonyControllerType ControllerType) = 0;

	/**
	 * Destroy audio device
	 */
	virtual void	TearDownAudio() = 0;

	/**
	 * Push audio frame to endpoint on device
	 */
	virtual void	PushAudio(EWinDualShockPortType PortType, const TArrayView<const float>& InAudio, const int32& NumChannels) = 0;

	/**
	 * Endpoint reference counting to know when device is no longer needed
	 * Returns true only for the first endpoint of a port type, to avoid multiple audio sends interleaving
	 */
	bool AddEndpoint(EWinDualShockPortType PortType)
	{
		int PreviousCount = EndpointCount[PortType]++;
		return PreviousCount == 0;
	}
	void RemoveEndpoint(EWinDualShockPortType PortType)
	{
		EndpointCount[PortType]--;
	}
	int GetEndpointCount() const
	{
		static_assert(WinDualShockPortType_Count == 2);
		return EndpointCount[0] + EndpointCount[1];
	}

	FDeviceKey GetDeviceKey() const
	{
		return DeviceKey;
	}

protected:
	FDeviceKey			DeviceKey;
	std::atomic<int>	EndpointCount[WinDualShockPortType_Count]{ 0 };
};


enum EWinDualShockDefaults
{
	SampleRate = 48000,
	NumFrames = 256,
	PadSpeakerChannels = 2,
	VibrationChannels = 2,
	MicrophoneChannels = 2,
	QueueDepth = 4
};
