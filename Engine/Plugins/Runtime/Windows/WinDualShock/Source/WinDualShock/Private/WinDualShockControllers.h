// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if DUALSHOCK4_SUPPORT

#include "WinDualShock.h"
#include LIBSCEPAD_PLATFORM_INCLUDE

THIRD_PARTY_INCLUDES_START
	#include <pad.h>
	#include <pad_audio.h>
THIRD_PARTY_INCLUDES_END

#include "GameFramework/InputDeviceSubsystem.h"

class FWinDualShockControllers : public FPlatformControllers
{
public:

	FWinDualShockControllers()
		: FPlatformControllers()
	{
	}

	virtual ~FWinDualShockControllers() = default;

	void SetAudioGain(float InPadSpeakerGain, float InHeadphonesGain, float InMicrophoneGain, float InOutputGain)
	{
		PadSpeakerGain = InPadSpeakerGain;
		HeadphonesGain = InHeadphonesGain;
		MicrophoneGain = InMicrophoneGain;
		OutputGain = InOutputGain;
		for (int32 UserIndex = 0; UserIndex < SCE_USER_SERVICE_MAX_LOGIN_USERS; UserIndex++)
		{
			bGainChanged[UserIndex] = true;
		}
	}

	float GetOutputGain()
	{
		return OutputGain;
	}

	bool GetSupportsAudio(int32 UserIndex) const
	{
		return bSupportsAudio[UserIndex];
	}

	void RefreshControllerType(int32 UserIndex)
	{
		ControllerTypeIdentifiers[UserIndex] = GetControllerType(UserIndex);
	}
	
	/**
	 * Returns true if the most recently used FInputDeviceId for the given ControllerId
	 * is "owned" by this input interface.  
	 */
	bool OwnsMostRecentlyUsedDevice(int32 ControllerId) const
	{
		// Find the most recently used input device by the given platform user
		const FPlatformUserId UserId = IPlatformInputDeviceMapper::Get().GetPlatformUserForUserIndex(ControllerId);
		UInputDeviceSubsystem* DeviceSubsystem = UInputDeviceSubsystem::Get();
		check(DeviceSubsystem);
		const FInputDeviceId MostRecentDevice = DeviceSubsystem->GetMostRecentlyUsedInputDeviceId(UserId);

		// If that is one of our controllers, then process our force feedback!
		for (int32 UserIndex=0; UserIndex < SCE_USER_SERVICE_MAX_LOGIN_USERS; ++UserIndex)
		{
			const FControllerState& ControllerState = ControllerStates[UserIndex];

			// Note: This will only work for "normal" pad devices, not special devices. That is ok
			// because windows does not support special device handles at all
			const FInputDeviceId OurDeviceId = InternalDeviceIdMappings.FindDeviceId(ControllerState.Handle);
			if (OurDeviceId == MostRecentDevice)
			{
				return true;	
			}
		}
		
		return false;
	}

	virtual void SetForceFeedbackChannelValue(int32 ControllerId, FForceFeedbackChannelType ChannelType, float Value) override
	{
		// If that is one of our controllers, then process our force feedback! 	
		if (!OwnsMostRecentlyUsedDevice(ControllerId))
		{
			return;
		}
		
		FPlatformControllers::SetForceFeedbackChannelValue(ControllerId, ChannelType, Value);
	}
	
	virtual void SetForceFeedbackChannelValues(int32 ControllerId, const FForceFeedbackValues &Values) override
	{
		// If that is one of our controllers, then process our force feedback! 	
		if (!OwnsMostRecentlyUsedDevice(ControllerId))
		{
			return;
		}
		
		FPlatformControllers::SetForceFeedbackChannelValues(ControllerId, Values);
	}
	
	

private:
	float OutputGain = 1.0f;
};

#endif // DUALSHOCK4_SUPPORT
