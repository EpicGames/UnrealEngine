// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if DUALSHOCK4_SUPPORT

#include "AudioMixer.h"
#include "IAudioEndpoint.h"
#include "WinDualShock.h"
#include "WinDualShockSettings.h"

/**
 * these endpoints can be used to send audio to arbitrary virtual audio devices supported by the PS5.
 */
template<EWinDualShockPortType PortType>
class FExternalWinDualShockEndpoint : public IAudioEndpoint
{
private:
	TSharedRef<IWinDualShockAudioDevice> Device;
	bool bAllowedEndpoint;

public:
	FExternalWinDualShockEndpoint(TSharedRef<IWinDualShockAudioDevice> InDevice) 
		: Device(InDevice), bAllowedEndpoint(InDevice->AddEndpoint(PortType))
	{
	}

	virtual ~FExternalWinDualShockEndpoint()
	{
		Device->RemoveEndpoint(PortType);
	}

	bool IsEndpointAllowed() const
	{
		return bAllowedEndpoint;
	}

protected:
	virtual float GetSampleRate() const override
	{
		return EWinDualShockDefaults::SampleRate;
	}

	virtual int32 GetNumChannels() const override
	{
		if (PortType == EWinDualShockPortType::Vibration)
		{
			return EWinDualShockDefaults::VibrationChannels;
		}
		else if (PortType == EWinDualShockPortType::PadSpeakers)
		{
			return EWinDualShockDefaults::PadSpeakerChannels;
		}
		else
		{
			checkNoEntry();
			return 0;
		}
	}

	virtual bool EndpointRequiresCallback() const override
	{
		return true;
	}

	virtual int32 GetDesiredNumFrames() const override
	{
		return EWinDualShockDefaults::NumFrames;
	}

	virtual bool OnAudioCallback(const TArrayView<const float>& InAudio, const int32& NumChannels, const IAudioEndpointSettingsProxy* InSettings) override
	{
		check(NumChannels == GetNumChannels());

		if (bAllowedEndpoint)
		{
			Device->PushAudio(PortType, InAudio, NumChannels);
		}
		return true;
	}

	virtual bool IsImplemented() override
	{
		return true;
	}
};

#endif // DUALSHOCK4_SUPPORT
