// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioMixer.h"
#include "ISoundfieldFormat.h"
#include "ISoundfieldEndpoint.h"
#include "IAudioEndpoint.h"
#include "WinDualShockSettingsProxies.h"
#include "WinDualShockSettings.generated.h"

UCLASS()
class UDualShockExternalEndpointSettings : public UAudioEndpointSettingsBase
{
	GENERATED_BODY()

public:

	UPROPERTY(Category = Output, EditAnywhere)
	int32 ControllerIndex;

	virtual TUniquePtr<IAudioEndpointSettingsProxy> GetProxy() const override
	{
		FDualShockExternalEndpointSettings* Settings = new FDualShockExternalEndpointSettings();
		Settings->ControllerIndex = ControllerIndex;
		return TUniquePtr<IAudioEndpointSettingsProxy>(Settings);
	}
};

UCLASS()
class UDualShockSoundfieldEndpointSettings : public USoundfieldEndpointSettingsBase
{
	GENERATED_BODY()

public:

	UPROPERTY(Category = Output, EditAnywhere)
	int32 ControllerIndex;

	virtual TUniquePtr<ISoundfieldEndpointSettingsProxy> GetProxy() const override
	{
		FDualShockSoundfieldEndpointSettings* Settings = new FDualShockSoundfieldEndpointSettings();
		Settings->ControllerIndex = ControllerIndex;
		return TUniquePtr<ISoundfieldEndpointSettingsProxy>(Settings);
	}
};

UCLASS()
class UDualShockSpatializationSettings : public USoundfieldEncodingSettingsBase
{
	GENERATED_BODY()
public:
	UPROPERTY(Category = Output, EditAnywhere, meta=(ClampMin = "0", ClampMax = "6.2831"))
	float Spread = 1.0f;

	UPROPERTY(Category = Output, EditAnywhere, meta = (ClampMin = "0", ClampMax = "1000"))
	int32 Priority = 0;

	UPROPERTY(Category = Output, EditAnywhere)
	bool Passthrough = false;

	virtual TUniquePtr<ISoundfieldEncodingSettingsProxy> GetProxy() const override
	{
		FDualShockSpatializationSettings* Settings = new FDualShockSpatializationSettings();
		Settings->Spread = Spread;
		Settings->Priority = Priority;
		Settings->Passthrough = Passthrough;

		return TUniquePtr<ISoundfieldEncodingSettingsProxy>(Settings);
	}
};
