// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioMixer.h"
#include "ISoundfieldFormat.h"
#include "ISoundfieldEndpoint.h"
#include "IAudioEndpoint.h"

class FDualShockExternalEndpointSettings : public IAudioEndpointSettingsProxy
{
public:
	int32 ControllerIndex = INDEX_NONE;
};

class FDualShockSoundfieldEndpointSettings : public ISoundfieldEndpointSettingsProxy
{
public:
	int32 ControllerIndex = INDEX_NONE;
};

class FDualShockSpatializationSettings : public ISoundfieldEncodingSettingsProxy
{

public:
	float Spread;
	int32 Priority;
	bool Passthrough;

	virtual uint32 GetUniqueId() const override
	{
		return FCrc::MemCrc32(this, sizeof(*this));
	}


	virtual TUniquePtr<ISoundfieldEncodingSettingsProxy> Duplicate() const override
	{
		return TUniquePtr<ISoundfieldEncodingSettingsProxy>(new FDualShockSpatializationSettings(*this));
	}
};
