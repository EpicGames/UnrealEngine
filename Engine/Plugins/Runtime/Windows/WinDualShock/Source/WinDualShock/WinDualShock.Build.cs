// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Reflection;

namespace UnrealBuildTool.Rules
{
	public class WinDualShock : ModuleRules
	{
		public WinDualShock(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"ApplicationCore",
					"Engine",
					"Slate",
					"InputDevice",
					"AudioMixerCore",
					"AudioMixer",
					"AudioExtensions",
					"Projects"
			});

			// Use reflection to allow type not to exist if console code is not present
			System.Type LibScePadType = System.Type.GetType("LibScePad");
			bool bHasSupport = false;
			string PlatformName = "";
			if (LibScePadType != null && Target.Architecture != UnrealArch.Arm64)
			{
				MethodInfo GetPadLibLocationMethod = LibScePadType.GetMethod("GetPadLibLocation");
				object[] GetPadLibLocationMethodParams = new object[] { EngineDirectory, Target.WindowsPlatform.Compiler, null };
				bHasSupport = (bool)GetPadLibLocationMethod.Invoke(null, GetPadLibLocationMethodParams);
				PlatformName = (string)LibScePadType.GetMethod("GetPlatformName").Invoke(null, new object[] { EngineDirectory, Target.WindowsPlatform.Compiler });
				if (bHasSupport)
				{
					ExtraRootPath = ("LibScePad", (string)GetPadLibLocationMethodParams[2]);

					AddEngineThirdPartyPrivateStaticDependencies(Target,
						"LibScePad",
						"XAudio2_9"
					);
					string[] Includes = new string[2];
					Includes[0] = "ApplicationCore_Sony";
					Includes[1] = "ApplicationCore_" + PlatformName;
					PrivateIncludePathModuleNames.AddRange(Includes);
				}
			}

			PublicDefinitions.Add("DUALSHOCK4_SUPPORT=" + (bHasSupport ? "1" : "0"));
			PublicDefinitions.Add("DUALSENSE_SUPPORT=" + (PlatformName.Equals("PS5") ? "1" : "0"));
		}
	}
}