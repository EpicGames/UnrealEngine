{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Windows DualShock",
	"Description" : "InputDevice plugin for the PS4 DualShock controller in Windows",
	"Category" : "Input Devices",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"RequiresBuildPlatform" : false,
	
	"SupportedTargetPlatforms": [
		"Win64"
	],
	
    "Modules" :
	[
		{
			"Name" : "WinDualShock",
			"Type" : "RuntimeNoCommandlet",
			"LoadingPhase" : "Default",
			"PlatformAllowList" :
			[
				"Win64"
			]
		}
	]
}